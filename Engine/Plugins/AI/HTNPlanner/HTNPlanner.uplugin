{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.01",
	"FriendlyName" : "HTN planner",
	"Description" : "[EXPERIMENTAL] Adds experimental support for Hierarchical Task Network (HTN) planner to the UE4's AI module",
	"Category" : "AI",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
        {
            "Name": "HTNPlanner",
            "Type": "Runtime",
            "LoadingPhase": "PreDefault"
        },
        {
			"Name" : "HTNTestSuite",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		}
	]
}