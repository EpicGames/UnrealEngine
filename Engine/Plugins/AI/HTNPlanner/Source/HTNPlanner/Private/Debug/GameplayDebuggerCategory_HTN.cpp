// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/GameplayDebuggerCategory_HTN.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

FGameplayDebuggerCategory_HTN::FGameplayDebuggerCategory_HTN()
{
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_HTN::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_HTN());
}

void FGameplayDebuggerCategory_HTN::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
}

void FGameplayDebuggerCategory_HTN::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{

}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
