// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/HTNDebug.h"
#include "HTNBuilder.h"
#include "UObject/Class.h"

namespace FHTNDebug
{
	FString HTNWorldStateCheckToString(const EHTNWorldStateCheck Value)
	{
		static const UEnum* EnumType = StaticEnum<EHTNWorldStateCheck>();
		return EnumType->GetNameStringByIndex(static_cast<uint32>(Value));
	}

	FString HTNWorldStateOperationToString(const EHTNWorldStateOperation Value)
	{
		static const UEnum* EnumType = StaticEnum<EHTNWorldStateOperation>();
		return EnumType->GetNameStringByIndex(static_cast<uint32>(Value));
	}

	FName GetTaskName(const FHTNDomain& Domain, const FHTNPolicy::FTaskID TaskID)
	{
		return Domain.GetTaskName(TaskID);
	}

	FName GetTaskName(const FHTNBuilder_Domain& DomainBuilder, const FHTNPolicy::FTaskID TaskID)
	{
		return DomainBuilder.DomainInstance->GetTaskName(TaskID);
	}

	FString GetDescription(const FHTNDomain& Domain, const TArray<FHTNPolicy::FTaskID>& TaskIDs, const FString& Delimiter, const int32 CurrentTaskIndex)
	{
		FString Description;
#if WITH_HTN_DEBUG
		for (int32 TaskIndex = 0; TaskIndex < TaskIDs.Num(); ++TaskIndex)
		{
			const FHTNPolicy::FTaskID TaskID = TaskIDs[TaskIndex];
			Description += FString::Printf(TEXT("%s%s%s")
				, (CurrentTaskIndex == TaskIndex) ? TEXT("* ") : TEXT("")
				, *Domain.GetTaskName(TaskID).ToString()
				, *Delimiter
			);
		}
#else
		for (int32 TaskIndex = 0; TaskIndex < TaskIDs.Num(); ++TaskIndex)
		{
			const FHTNPolicy::FTaskID TaskID = TaskIDs[TaskIndex];
			Description += FString::Printf(TEXT("%s%d%s")
				, (CurrentTaskIndex == TaskIndex) ? TEXT("* ") : TEXT("")
				, int32(TaskID)
				, *Delimiter
			);
		}
#endif // WITH_HTN_DEBUG
		return Description;
	}

	/*FString GetDescription(const TArray<FHTNPolicy::FTaskID>& TaskIDs, const FString& Delimiter, const UEnum* TaskEnum, const int32 CurrentTaskIndex)
	{
		FString Description;

		if (TaskEnum)
		{
			if (CurrentTaskIndex != INDEX_NONE)	
			{
				for (int32 TaskIndex = 0; TaskIndex < TaskIDs.Num(); ++TaskIndex)
				{
					const FHTNPolicy::FTaskID TaskID = TaskIDs[TaskIndex];
					Description += FString:Printf(TEXT("%s%s")
						, TaskEnum->GetEnumName(static_cast<uint32>(TaskID))
						, *Delimiter
					);
				}
			}
		}
		else
		{

		}

		return Description;
	}*/
}
