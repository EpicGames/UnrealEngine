// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTNBuilder.h"
#include "Debug/HTNDebug.h"

//----------------------------------------------------------------------//
// FHTNDomainBuilder
//----------------------------------------------------------------------//
FHTNBuilder_Domain::FHTNBuilder_Domain()
	: DomainInstance(new FHTNDomain())
	, RootTaskName(NAME_None)
{

}

FHTNBuilder_Domain::FHTNBuilder_Domain(const TSharedPtr<FHTNDomain>& InDomain)
	: DomainInstance(InDomain)
{

}

FHTNBuilder_CompositeTask& FHTNBuilder_Domain::AddCompositeTask(const FName& TaskName)
{
	return CompositeTasks.FindOrAdd(TaskName);
}

FHTNBuilder_PrimitiveTask& FHTNBuilder_Domain::AddPrimitiveTask(const FName& TaskName)
{
	return PrimitiveTasks.FindOrAdd(TaskName);
}

FHTNBuilder_PrimitiveTask* FHTNBuilder_Domain::FindPrimitiveTask(const FName& TaskName)
{
	return PrimitiveTasks.Find(TaskName);
}

FHTNBuilder_CompositeTask* FHTNBuilder_Domain::FindCompositeTask(const FName& TaskName)
{
	return CompositeTasks.Find(TaskName);
}

bool FHTNBuilder_Domain::Compile()
{
	FHTNDomain& Domain = *DomainInstance;
	Domain.Reset();

	// quell static analyzer complaint
	uint32 MemoryRequired = 16;
	for (auto It = PrimitiveTasks.CreateConstIterator(); It; ++It)
	{
		const FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = It->Value;
		MemoryRequired += sizeof(FHTNPrimitiveTask);
		MemoryRequired += sizeof(FHTNEffect)* PrimitiveTaskBuilder.Effects.Num();
	}

	for (auto It = CompositeTasks.CreateIterator(); It; ++It)
	{
		/*const */FHTNBuilder_CompositeTask& CompositeTaskBuilder = It->Value;
		MemoryRequired += sizeof(FHTNCompositeTask);
		MemoryRequired += sizeof(FHTNMethod) * CompositeTaskBuilder.Methods.Num();
		
		for (/*const */FHTNBuilder_Method& Method : CompositeTaskBuilder.Methods)
		{
			MemoryRequired += sizeof(FHTNPolicy::FTaskID) * Method.Tasks.Num();
			MemoryRequired += sizeof(FHTNCondition) * Method.Conditions.Num();
		}
	}

	constexpr std::size_t MemoryAlignment = FMath::Max(FMath::Max3(alignof(FHTNPrimitiveTask), alignof(FHTNEffect), alignof(FHTNMethod)),
		FMath::Max(alignof(FHTNPolicy::FTaskID), alignof(FHTNCondition)));

	uint8* CompiledData = static_cast<uint8*>(FMemory::Malloc(MemoryRequired, MemoryAlignment));
	uint8* CurrentAddress = CompiledData;
	
	for (auto It = PrimitiveTasks.CreateConstIterator(); It; ++It)
	{
		const FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = It->Value;
		FHTNPrimitiveTask* Task = new(CurrentAddress) FHTNPrimitiveTask();

		Domain.TaskNameMap.Add(It->Key, (CurrentAddress - CompiledData) / sizeof(uint8));

		Task->ActionID = IntCastChecked<FHTNPolicy::FActionID>(PrimitiveTaskBuilder.ActionID);
		Task->Parameter = PrimitiveTaskBuilder.Parameter;
		const int32 EffectsNum = PrimitiveTaskBuilder.Effects.Num();
		Task->EffectsCount = IntCastChecked<uint8>(EffectsNum);

		CurrentAddress += sizeof(FHTNPrimitiveTask);

		if (EffectsNum)
		{
			Task->Effects = (FHTNEffect*)CurrentAddress;
			FMemory::Memcpy(Task->Effects, PrimitiveTaskBuilder.Effects.GetData(), EffectsNum * sizeof(FHTNEffect));
			CurrentAddress += sizeof(FHTNEffect) * EffectsNum;
		}
	}

	Domain.FirstCompositeTaskID = (CurrentAddress - CompiledData) / sizeof(uint8);

	for (auto It = CompositeTasks.CreateConstIterator(); It; ++It)
	{
		const FHTNBuilder_CompositeTask& CompositeTaskBuilder = It->Value;
		FHTNCompositeTask* Task = new(CurrentAddress) FHTNCompositeTask();
		
		Domain.TaskNameMap.Add(It->Key, (CurrentAddress - CompiledData) / sizeof(uint8));

		const int32 MethodsCount = CompositeTaskBuilder.Methods.Num();
		Task->MethodsCount = IntCastChecked<uint8>(MethodsCount);

		CurrentAddress += sizeof(FHTNCompositeTask);

		if (MethodsCount > 0)
		{
			Task->Methods = (FHTNMethod*)CurrentAddress;
			
			CurrentAddress += sizeof(FHTNMethod) * MethodsCount;

			for (int32 MethodIndex = 0; MethodIndex < MethodsCount; ++MethodIndex)
			{
				const FHTNBuilder_Method& MethodBuilder = CompositeTaskBuilder.Methods[MethodIndex];

				FHTNMethod* Method = new (&Task->Methods[MethodIndex]) FHTNMethod();
				Method->ConditionsCount = IntCastChecked<uint8>(MethodBuilder.Conditions.Num());
				Method->TasksCount = IntCastChecked<uint8>(MethodBuilder.Tasks.Num());

				if (Method->ConditionsCount > 0)
				{
					Method->Conditions = (FHTNCondition*)CurrentAddress;
					FMemory::Memcpy(Method->Conditions, MethodBuilder.Conditions.GetData(), Method->ConditionsCount * sizeof(FHTNCondition));
					CurrentAddress += sizeof(FHTNCondition) * Method->ConditionsCount;
				}

				if (Method->TasksCount > 0)
				{
					Method->Tasks = (FHTNPolicy::FTaskID*)CurrentAddress;
					// @todo copying might not be needed, since we still need to populate this array
					//FMemory::Memcpy(Method->Conditions, MethodBuilder.Conditions.GetData(), Method->TasksCount * sizeof(FHTNPolicy::FTaskID));
					CurrentAddress += sizeof(FHTNPolicy::FTaskID) * Method->TasksCount;
				}
			}
		}
	}

	// assigning CompiledData to the Domain instance even though
	// at this point it's still unknown if the compilation will
	// be successful. This is done mostly to utilize FHTNDomain's 
	// API in the following loop. If compilation fails the 
	// RawData will get set back to nullptr at the end of this 
	// function
	Domain.RawData = CompiledData;
	uint32 MemoryUsed = (CurrentAddress - CompiledData) / sizeof(uint8);

	bool bIssuesFound = false;
	// patch up task IDs in compound tasks' methods
	for (auto It = CompositeTasks.CreateConstIterator(); It; ++It)
	{
		if (bIssuesFound)
		{
			break;
		}

		const FHTNBuilder_CompositeTask& CompositeTaskBuilder = It->Value;
		const FHTNPolicy::FTaskID CompositeTaskID = Domain.FindTaskID(It->Key);
		const FHTNCompositeTask& CompositeTask = Domain.GetCompositeTask(CompositeTaskID);

		for (int32 MethodIndex = 0; MethodIndex < CompositeTask.MethodsCount; ++MethodIndex)
		{
			const FHTNBuilder_Method& MethodBuilder = CompositeTaskBuilder.Methods[MethodIndex];
			FHTNMethod& Method = CompositeTask.Methods[MethodIndex];
			for (int32 TaskIndex = 0; TaskIndex < Method.TasksCount; ++TaskIndex)
			{
				const FName TaskName = MethodBuilder.Tasks[TaskIndex];
				FHTNPolicy::FTaskID TaskID = Domain.FindTaskID(TaskName);
				Method.Tasks[TaskIndex] = TaskID;
				bIssuesFound = (TaskID == FHTNPolicy::InvalidTaskID);
				UE_CLOGF(bIssuesFound && !GIsAutomationTesting, LogHTNPlanner, Warning
					, "Task %ls of %ls[%d] not found! Make sure it's defined in your domain. Domain compilation aborted."
					, *TaskName.ToString(), *It->Key.ToString(), MethodIndex);

			}
		}
	}

	if (bIssuesFound)
	{
		Domain.ReleaseRawData();
		CompiledData = nullptr;
	}
	else if (MemoryUsed > 0)	// meaning we have anything scored
	{
		Domain.TaskNameMap.Shrink();

		FHTNPolicy::FTaskID* RootTaskID = Domain.TaskNameMap.Find(RootTaskName);
		if (RootTaskID)
		{
			Domain.RootTaskID = *RootTaskID;
		}
		else
		{
			UE_CLOGF(!GIsAutomationTesting, LogHTNPlanner, Warning
				, "Unable to find root task under the name %ls. Falling back to the first compound task, or first primitive task"
				, *RootTaskName.ToString());
			if (Domain.FirstCompositeTaskID != FHTNPolicy::InvalidTaskID)
			{
				Domain.RootTaskID = Domain.FirstCompositeTaskID;
			}
			else
			{
				Domain.RootTaskID = 0;
			}
		}

#if WITH_HTN_DEBUG
		// Create reverse 
		for (auto It = Domain.TaskNameMap.CreateConstIterator(); It; ++It)
		{
			Domain.TaskIDToName.Add(It->Value, It->Key);
		}
		Domain.TaskIDToName.Shrink();
#endif // WITH_HTN_DEBUG
	}

	return (bIssuesFound == false);
}

void FHTNBuilder_Domain::Decompile()
{
	if (DomainInstance.IsValid() == false)
	{
		return;
	}

	const FHTNDomain& Domain = *DomainInstance;
	TMap<FHTNPolicy::FTaskID, FName> IDToName;
	
	// build ID-to-Name map	
	for (auto It = Domain.TaskNameMap.CreateConstIterator(); It; ++It)
	{
		IDToName.Add(It->Value, It->Key);

		if (Domain.IsPrimitiveTask(It->Value))
		{
			const FHTNPrimitiveTask& PrimitiveTask = Domain.GetPrimitiveTask(It->Value);
			FHTNBuilder_PrimitiveTask& TaskBuilder = AddPrimitiveTask(It->Key);
			TaskBuilder.ActionID = PrimitiveTask.ActionID;
			TaskBuilder.Parameter = PrimitiveTask.Parameter;
			for (int32 EffectIndex = 0; EffectIndex < PrimitiveTask.EffectsCount; ++EffectIndex)
			{
				TaskBuilder.AddEffect(PrimitiveTask.Effects[EffectIndex]);
			}
		}
		else
		{
			//const FHTNCompositeTask& CompositeTask = Domain.GetCompositeTask(It->Value);
			AddCompositeTask(It->Key); //-V530
		}
	}

	for (auto It = CompositeTasks.CreateIterator(); It; ++It)
	{
		FHTNBuilder_CompositeTask& CompositeTaskBuilder = It->Value;
		const FHTNPolicy::FTaskID TaskID = Domain.FindTaskID(It->Key);
		const FHTNCompositeTask& CompositeTask = Domain.GetCompositeTask(TaskID);

		for (int32 MethodIndex = 0; MethodIndex < CompositeTask.MethodsCount; ++MethodIndex)
		{
			FHTNBuilder_Method& MethodBuilder = CompositeTaskBuilder.AddMethod();
			FHTNMethod& Method = CompositeTask.Methods[MethodIndex];
			
			for (int32 ConditionIndex = 0; ConditionIndex < Method.ConditionsCount; ++ConditionIndex)
			{
				MethodBuilder.Conditions.Add(Method.Conditions[ConditionIndex]);
			}

			for (int32 TaskIndex = 0; TaskIndex < Method.TasksCount; ++TaskIndex)
			{
				const FName* TaskName = IDToName.Find(Method.Tasks[TaskIndex]);
				if (TaskName)
				{
					MethodBuilder.AddTask(*TaskName);
				}
				else
				{
					// @error!
				}
			}
		}
	}
}

FString FHTNBuilder_Domain::GetDebugDescription() const
{
	FString Description;
#if WITH_HTN_DEBUG	
	for (auto It = CompositeTasks.CreateConstIterator(); It; ++It)
	{
		Description += FString::Printf(TEXT("%s:\n"), *It->Key.ToString());
		for (const FHTNBuilder_Method& Method : It->Value.Methods)
		{
			if (Method.Conditions.Num() > 0)
			{
				FString ConditionsDesc = FString::Printf(TEXT("? %s"), *FHTNDebug::GetDescription(Method.Conditions[0]));
				for (int32 ConditionIndex = 1; ConditionIndex < Method.Conditions.Num(); ++ConditionIndex)
				{
					ConditionsDesc += FString::Printf(TEXT(" AND %s"), *FHTNDebug::GetDescription(Method.Conditions[ConditionIndex]));
				}
				Description += FString::Printf(TEXT("\t%s:\n"), *ConditionsDesc);
			}
			else
			{
				Description += TEXT("\t[conditionless]\n");
			}

			for (const FName& TaskName : Method.Tasks)
			{
				Description += FString::Printf(TEXT("\t\t%s\n"), *TaskName.ToString());
			}
		}
	}

	for (auto It = PrimitiveTasks.CreateConstIterator(); It; ++It)
	{
		Description += FString::Printf(TEXT("%s:\n"), *It->Key.ToString());
		Description += FString::Printf(TEXT("\tOp: %d param: %d\n"), It->Value.ActionID, It->Value.Parameter);

		if (It->Value.Effects.Num() > 0)
		{
			for (int32 EffectIndex = 1; EffectIndex < It->Value.Effects.Num(); ++EffectIndex)
			{
				Description += FString::Printf(TEXT("\t\t[%s]\n"), *FHTNDebug::GetDescription(It->Value.Effects[EffectIndex]));
			}
		}
		else
		{
			Description += TEXT("\t\t[no effect]\n");
		}
	}
#endif

	return Description;
}
