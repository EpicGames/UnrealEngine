// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTNDomain.h"
#include "Debug/HTNDebug.h"

DEFINE_LOG_CATEGORY(LogHTNPlanner);

//----------------------------------------------------------------------//
// Missing:
//	- store operation in an TArray
//	- custom, project specific operations
//----------------------------------------------------------------------//


//----------------------------------------------------------------------//
// FHTNWorldState
//----------------------------------------------------------------------//
FHTNWorldState::FHTNWorldState(const uint32 WorldStateSize)
{
	if (WorldStateSize)
	{
		Values.AddZeroed(WorldStateSize);
	}
}

FHTNWorldState::~FHTNWorldState()
{
	
}

void FHTNWorldState::Reinit(const uint32 NewWorldStateSize)
{
	Values.Reset(NewWorldStateSize);
	Values.AddZeroed(NewWorldStateSize);
}

namespace FHTNWorldStateOperations
{
	template<typename TOperation, typename TValues>
	FORCEINLINE const FHTNPolicy::FWSValue GetValue(TValues* Values, const TOperation& Operation)
	{
		return Operation.IsRHSAbsolute() ? Operation.Value : Values[Operation.KeyRightHand];
	}

	bool CheckLess(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] < GetValue(Values, Condition);
	}

	bool CheckLessOrEqual(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] <= GetValue(Values, Condition);
	}

	bool CheckEqual(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] == GetValue(Values, Condition);
	}

	bool CheckNotEqual(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] != GetValue(Values, Condition);
	}

	bool CheckGreaterOrEqual(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] >= GetValue(Values, Condition);
	}

	bool CheckGreater(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] > GetValue(Values, Condition);
	}

	bool CheckIsTrue(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		return Values[Condition.KeyLeftHand] != 0;
	}

	/*FConditionFunctionPtr ConditionFunctions2[uint8(EHTNWorldStateCheck::MAX)] = {
		&CheckLess
		, &CheckLessOrEqual
		, &CheckEqual
		, &CheckNotEqual
		, &CheckGreaterOrEqual
		, &CheckGreater
		, &CheckIsTrue
	};*/
	
	void OpSet(FHTNPolicy::FWSValue* Values, const FHTNEffect& Effect)
	{
		Values[Effect.KeyLeftHand] = GetValue(Values, Effect);
	}

	void OpInc(FHTNPolicy::FWSValue* Values, const FHTNEffect& Effect)
	{
		Values[Effect.KeyLeftHand] += GetValue(Values, Effect);
	}

	void OpDec(FHTNPolicy::FWSValue* Values, const FHTNEffect& Effect)
	{
		Values[Effect.KeyLeftHand] -= GetValue(Values, Effect);
	}

	TArray<FOperationFunctionPtr> OpFunctions;
	TArray<FConditionFunctionPtr> CheckFunctions;
#if WITH_HTN_DEBUG
	TArray<FName> OpNames;
	TArray<FName> CheckNames;
#endif // WITH_HTN_DEBUG

	struct FWSOperationsSetUp
	{
		FWSOperationsSetUp()
		{
			OpFunctions.Reserve(uint8(EHTNWorldStateOperation::MAX));
			ensure(OpFunctions.Add(&OpSet) == int32(EHTNWorldStateOperation::Set));
			ensure(OpFunctions.Add(&OpInc) == int32(EHTNWorldStateOperation::Increase));
			ensure(OpFunctions.Add(&OpDec) == int32(EHTNWorldStateOperation::Decrease));

			CheckFunctions.Reserve(uint8(EHTNWorldStateCheck::MAX));
			ensure(CheckFunctions.Add(&CheckLess) == int32(EHTNWorldStateCheck::Less));
			ensure(CheckFunctions.Add(&CheckLessOrEqual) == int32(EHTNWorldStateCheck::LessOrEqual));
			ensure(CheckFunctions.Add(&CheckEqual) == int32(EHTNWorldStateCheck::Equal));
			ensure(CheckFunctions.Add(&CheckNotEqual) == int32(EHTNWorldStateCheck::NotEqual));
			ensure(CheckFunctions.Add(&CheckGreaterOrEqual) == int32(EHTNWorldStateCheck::GreaterOrEqual));
			ensure(CheckFunctions.Add(&CheckGreater) == int32(EHTNWorldStateCheck::Greater));
			ensure(CheckFunctions.Add(&CheckIsTrue) == int32(EHTNWorldStateCheck::IsTrue));

#if WITH_HTN_DEBUG
			OpNames.SetNum(uint8(EHTNWorldStateOperation::MAX));
			OpNames[int32(EHTNWorldStateOperation::Set)] = TEXT("Set");
			OpNames[int32(EHTNWorldStateOperation::Increase)] = TEXT("Increase");
			OpNames[int32(EHTNWorldStateOperation::Decrease)] = TEXT("Decrease");

			CheckNames.SetNum(uint8(EHTNWorldStateCheck::MAX));
			CheckNames[int32(EHTNWorldStateCheck::Less)] = TEXT("Less");
			CheckNames[int32(EHTNWorldStateCheck::LessOrEqual)] = TEXT("LessOrEqual");
			CheckNames[int32(EHTNWorldStateCheck::Equal)] = TEXT("Equal");
			CheckNames[int32(EHTNWorldStateCheck::NotEqual)] = TEXT("NotEqual");
			CheckNames[int32(EHTNWorldStateCheck::GreaterOrEqual)] = TEXT("GreaterOrEqual");
			CheckNames[int32(EHTNWorldStateCheck::Greater)] = TEXT("Greater");
			CheckNames[int32(EHTNWorldStateCheck::IsTrue)] = TEXT("IsTrue");
#endif // WITH_HTN_DEBUG
		}
	};
	const FWSOperationsSetUp Setup;

	FHTNPolicy::FWSOperationID RegisterCustomCheckType(const FConditionFunctionPtr CustomConditionFunctionPtr, const FName& DebugName)
	{
		int32 OperationID = CheckFunctions.Add(CustomConditionFunctionPtr);
		ensure(OperationID != INDEX_NONE);
#if WITH_HTN_DEBUG
		ensure(OperationID == CheckNames.Add(DebugName));
#endif // WITH_HTN_DEBUG
		return FHTNPolicy::FWSOperationID(OperationID);
	}

	FHTNPolicy::FWSOperationID RegisterCustomOperationType(const FOperationFunctionPtr CustomOperationFunctionPtr, const FName& DebugName)
	{
		int32 OperationID = OpFunctions.Add(CustomOperationFunctionPtr);
		ensure(OperationID != INDEX_NONE);
#if WITH_HTN_DEBUG
		ensure(OperationID == OpNames.Add(DebugName));
#endif // WITH_HTN_DEBUG
		return FHTNPolicy::FWSOperationID(OperationID);
	}
}

#if WITH_HTN_DEBUG
namespace FHTNDebug
{
	FString GetDescription(const FHTNCondition& Condition)
	{
		return Condition.IsRHSAbsolute()
			? FString::Printf(TEXT("[%d] %s %d"), Condition.KeyLeftHand, *FHTNWorldStateOperations::CheckNames[Condition.Operation].ToString(), Condition.Value)
			: FString::Printf(TEXT("[%d] %s [%d]"), Condition.KeyLeftHand, *FHTNWorldStateOperations::CheckNames[Condition.Operation].ToString(), Condition.KeyRightHand);
	}

	FString GetDescription(const FHTNEffect& Effect)
	{
		return Effect.IsRHSAbsolute()
			? FString::Printf(TEXT("[%d] %s %d"), Effect.KeyLeftHand, *FHTNWorldStateOperations::OpNames[Effect.Operation].ToString(), Effect.Value)
			: FString::Printf(TEXT("[%d] %s [%d]"), Effect.KeyLeftHand, *FHTNWorldStateOperations::OpNames[Effect.Operation].ToString(), Effect.KeyRightHand);
	}
}
#endif // WITH_HTN_DEBUG

bool FHTNWorldState::CheckCondition(const FHTNCondition& Condition) const
{
	return (*(FHTNWorldStateOperations::CheckFunctions[Condition.Operation]))(Values.GetData(), Condition);
}

void FHTNWorldState::ApplyEffect(const FHTNEffect& Effect)
{
	(*(FHTNWorldStateOperations::OpFunctions[Effect.Operation]))(Values.GetData(), Effect);
}

bool FHTNWorldState::GetValue(const FHTNPolicy::FWSKey Key, FHTNPolicy::FWSValue& OutValue) const
{
	if (Key < FHTNPolicy::FWSKey(Values.Num()))
	{
		OutValue = Values[Key];
		return true;
	}
	
	return false;
}

bool FHTNWorldState::SetValue(const FHTNPolicy::FWSKey Key, const FHTNPolicy::FWSValue InValue)
{
	if (Key < FHTNPolicy::FWSKey(Values.Num()))
	{
		Values[Key] = InValue;
		return true;
	}

	return false;
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
//#pragma warning( push )
////#pragma warning( disable : 4582 )
////#pragma warning( disable : 4583 )
//FHTNTaskWrapper::FHTNTaskWrapper(const FHTNTaskWrapper& Other)
//	: bIsPrimitiveTask(0)
//{
//	FMemory::Memcpy(*this, Other);
//}
//
//FHTNTaskWrapper::~FHTNTaskWrapper()
//{
//	CompositeTask.~FHTNCompositeTask();
//}
//#pragma warning( pop )

int32 FHTNCompositeTask::FindSatisfiedMethod(const FHTNWorldState& WorldState, const int32 StartIndex) const
{
	for (int32 MethodIndex = StartIndex; MethodIndex < MethodsCount; ++MethodIndex)
	{
		if (WorldState.CheckConditions(Methods[MethodIndex].Conditions, Methods[MethodIndex].ConditionsCount))
		{
			return MethodIndex;
		}
	}

	return INDEX_NONE;
}

//----------------------------------------------------------------------//
// FHTNDomain
//----------------------------------------------------------------------//
FHTNDomain::FHTNDomain()
{
	RawData = nullptr;
	FirstCompositeTaskID = FHTNPolicy::InvalidTaskID;
	RootTaskID = FHTNPolicy::InvalidTaskID;
}

FHTNDomain::~FHTNDomain()
{
	ReleaseRawData();
}

void FHTNDomain::ReleaseRawData()
{
	if (RawData)
	{
		FMemory::Free(RawData);
		RawData = nullptr;
	}
}

void FHTNDomain::Reset()
{
	//Tasks.Reset();
	ReleaseRawData();
	TaskNameMap.Reset();
	FirstCompositeTaskID = FHTNPolicy::InvalidTaskID;
	RootTaskID = FHTNPolicy::InvalidTaskID;
#if WITH_HTN_DEBUG
	TaskIDToName.Reset();
#endif
}

FHTNPolicy::FTaskID FHTNDomain::FindTaskID(const FName& TaskName) const
{
	const FHTNPolicy::FTaskID* TaskID = TaskNameMap.Find(TaskName);
	return TaskID ? *TaskID : FHTNPolicy::InvalidTaskID;
}

FName FHTNDomain::GetTaskName(const FHTNPolicy::FTaskID TaskID) const
{
#if WITH_HTN_DEBUG
	const FName* Name = TaskIDToName.Find(TaskID);
	return Name ? *Name : NAME_None;
#else 
	return NAME_None;
#endif // WITH_HTN_DEBUG
}
