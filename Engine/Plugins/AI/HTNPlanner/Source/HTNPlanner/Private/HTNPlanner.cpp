// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTNPlanner.h"
#include "Stats/Stats.h"

DECLARE_CYCLE_STAT(TEXT("HTN: Planning"), STAT_HTN_Planning, STATGROUP_AI);

//
//	Extensions/missing features:
//	- PrimitiveTasks do not support conditions
//	- no planning progress logging
//	- max iterations limit

//----------------------------------------------------------------------//
// Planner
//----------------------------------------------------------------------//
FHTNPlanner::FHTNPlanner()
{

}

bool FHTNPlanner::GeneratePlan(const FHTNDomain& Domain, const FHTNWorldState& InitialWorldState, FHTNResult& Result, const FName& StartTaskName)
{
	SCOPE_CYCLE_COUNTER(STAT_HTN_Planning);

	const FHTNPolicy::FTaskID StartTaskID = StartTaskName == NAME_None ? Domain.GetRootTaskID() : Domain.FindTaskID(StartTaskName);

	Result.Reset();
	Reset();
	new(&CurrentState) FHTNRestorePoint(InitialWorldState);

	if (StartTaskID == FHTNPolicy::InvalidTaskID)
	{
		// @todo log an error
		UE_CLOGF(!GIsAutomationTesting, LogHTNPlanner, Warning, "Unable to find start task %ls. Make sure your Domain is compiled before using it for planning.", *StartTaskName.ToString());
		return false;
	}

	TasksToProcess.Push(StartTaskID);
		
	int32 NextMethod = 0;

	while (TasksToProcess.Num() > 0)
	{
		//self.print_progress(final_plan, tasks_to_process)

		const FHTNPolicy::FTaskID CurrentTaskID = TasksToProcess.Pop(EAllowShrinking::No);
		
		if (CurrentTaskID == FHTNPolicy::InvalidTaskID)
		{
			// @todo log error! 
			break;
		}

		if (Domain.IsCompositeTask(CurrentTaskID))
		{
			const FHTNCompositeTask& CompositeTask = Domain.GetCompositeTask(CurrentTaskID);
			const int32 MethodIndex = CompositeTask.FindSatisfiedMethod(CurrentWorldState(), NextMethod);

			if (MethodIndex != INDEX_NONE)
			{
				const FHTNMethod& Method = CompositeTask.Methods[MethodIndex];
				RecordDecomposition(CurrentTaskID, MethodIndex);
				// reset the method counter for the following compound tasks
				// the only way we can get back to this compound task is by going
				// through the else statement below, which will result in assigning
				// NextMethod an appropriate value
				NextMethod = 0;

				for (int32 TaskIndex = Method.TasksCount - 1; TaskIndex >= 0; --TaskIndex)
				{
					TasksToProcess.Push(Method.Tasks[TaskIndex]);
				}
			}
			else if (CanRollBack())
			{
				RestoreDecomposition();
				NextMethod = GetMethodIndex() + 1;
				TasksToProcess.Push(GetActiveTask());
			}
		}
		else
		{
			const FHTNPrimitiveTask& PrimitiveTask = Domain.GetPrimitiveTask(CurrentTaskID);
			CurrentWorldState().ApplyEffects(PrimitiveTask.Effects, PrimitiveTask.EffectsCount);
			AddToPlan(CurrentTaskID);
            /*if current_task.check_condition(working_ws):
                working_ws.apply(current_task.effects)
                final_plan.append(current_task_name)
            else:
                current_task, next_method, working_ws, final_plan\
                                        = self.restore_last_decomposition()
                tasks_to_process.append(current_task.name)*/
		}
	}

	const bool bPlanningSuccessful = (TasksToProcess.Num() == 0);
	if (bPlanningSuccessful)
	{
		Result.Set(Domain, GetPlan());
	}

	return bPlanningSuccessful;
}

//----------------------------------------------------------------------//
// FHTNResult
//----------------------------------------------------------------------//
void FHTNResult::Set(const FHTNDomain& Domain, const TArray<FHTNPolicy::FTaskID>& TasksSequence)
{
	TaskIDs = TasksSequence;
	ActionsSequence.Reset();
	for (const FHTNPolicy::FTaskID TaskID : TasksSequence)
	{
		if (ensure(TaskID != FHTNPolicy::InvalidTaskID && Domain.IsPrimitiveTask(TaskID)))
		{
			ActionsSequence.Add(Domain.GetPrimitiveTask(TaskID));
		}
	}
}
