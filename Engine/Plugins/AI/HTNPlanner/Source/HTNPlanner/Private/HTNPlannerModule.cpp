// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTNPlannerModule.h"
//#include "EngineDefines.h"
//#include "Templates/SubclassOf.h"
//#include "AISystem.h"
//#include "VisualLogger/VisualLogger.h"

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebugger.h"
#include "Debug/GameplayDebuggerCategory_HTN.h"
#endif // WITH_GAMEPLAY_DEBUGGER

#define LOCTEXT_NAMESPACE "HTNPlanner"

class FHTNPlannerModule : public IHTNPlannerModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FHTNPlannerModule, HTNPlanner)

void FHTNPlannerModule::StartupModule()
{
#if WITH_GAMEPLAY_DEBUGGER
	IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
	GameplayDebuggerModule.RegisterCategory("HTN", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_HTN::MakeInstance), EGameplayDebuggerCategoryState::EnabledInGameAndSimulate, 1);
	GameplayDebuggerModule.NotifyCategoriesChanged();
#endif
}

void FHTNPlannerModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

#if WITH_GAMEPLAY_DEBUGGER
	if (IGameplayDebugger::IsAvailable())
	{
		IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
		GameplayDebuggerModule.UnregisterCategory("HTN");
		GameplayDebuggerModule.NotifyCategoriesChanged();
	}
#endif
}

#undef LOCTEXT_NAMESPACE
