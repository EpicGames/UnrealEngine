// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "BrainComponent.h"
#include "HTNPlanner.h"
#include "HTNBrainComponent.generated.h"

UCLASS()
class HTNPLANNER_API UHTNBrainComponent : public UBrainComponent
{
	GENERATED_BODY()

public:
	UHTNBrainComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	FHTNPlanner Planner;
};