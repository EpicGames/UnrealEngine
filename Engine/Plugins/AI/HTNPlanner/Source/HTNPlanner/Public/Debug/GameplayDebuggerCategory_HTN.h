// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"

class ABotObjectiveGraph;

class FGameplayDebuggerCategory_HTN : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_HTN();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;

	static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
