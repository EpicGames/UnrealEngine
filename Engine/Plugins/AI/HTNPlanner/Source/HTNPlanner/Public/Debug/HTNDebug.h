// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HTNDomain.h"

struct FHTNBuilder_Domain;
struct FHTNDomain;

#if WITH_HTN_DEBUG
namespace FHTNDebug
{
	FString HTNPLANNER_API HTNWorldStateCheckToString(const EHTNWorldStateCheck Value);
	FString HTNPLANNER_API HTNWorldStateOperationToString(const EHTNWorldStateOperation Value);

	FString HTNPLANNER_API GetDescription(const FHTNCondition& Condition);
	FString HTNPLANNER_API GetDescription(const FHTNEffect& Effect);
	FString HTNPLANNER_API GetDescription(const FHTNDomain& Domain, const TArray<FHTNPolicy::FTaskID>& TaskIDs, const FString& Delimiter = TEXT(", "), const int32 CurrentTaskIndex = INDEX_NONE);

	FName HTNPLANNER_API GetTaskName(const FHTNBuilder_Domain& DomainBuilder, const FHTNPolicy::FTaskID TaskID);
	FName HTNPLANNER_API GetTaskName(const FHTNDomain& Domain, const FHTNPolicy::FTaskID TaskID);
}
#endif // WITH_HTN_DEBUG
