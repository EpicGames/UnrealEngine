// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HTNDomain.h"

struct HTNPLANNER_API FHTNBuilder_PrimitiveTask
{
	uint32 ActionID;
	uint32 Parameter;
	TArray<FHTNEffect> Effects;
	
	void SetOperator(const uint32 InActionID, const uint32 InParameter = 0)
	{
		ActionID = InActionID;
		Parameter = InParameter;
	}

	template<typename A, typename B=uint32>
	void SetOperator(const A InActionID, const B InParameter = B(0))
	{
		SetOperator(uint32(InActionID), uint32(InParameter));
	}
	
	void AddEffect(const FHTNEffect& Effect) { Effects.Add(Effect); }
};

struct HTNPLANNER_API FHTNBuilder_Method
{
	TArray<FHTNCondition> Conditions;
	TArray<FName> Tasks;

	FHTNBuilder_Method()
	{}
	explicit FHTNBuilder_Method(const FHTNCondition& InCondition)
	{
		if (ensure(InCondition.IsValid()))
		{
			Conditions.Add(InCondition);
		}
	}
	FHTNBuilder_Method(const TArray<FHTNCondition>& InConditions)
		: Conditions(InConditions)
	{}
	void AddTask(const FName& TaskName) { Tasks.Add(TaskName); }
};

struct HTNPLANNER_API FHTNBuilder_CompositeTask
{
	TArray<FHTNBuilder_Method> Methods;

	FHTNBuilder_Method& AddMethod()
	{
		return Methods[Methods.Add(FHTNBuilder_Method())];
	}

	FHTNBuilder_Method& AddMethod(const FHTNCondition& Condition)
	{
		return Methods[Methods.Add(FHTNBuilder_Method(Condition))];
	}

	FHTNBuilder_Method& AddMethod(const TArray<FHTNCondition>& Conditions)
	{
		return Methods[Methods.Add(FHTNBuilder_Method(Conditions))];
	}
};

struct HTNPLANNER_API FHTNBuilder_Domain// : public FHTNDomain
{
	TSharedPtr<FHTNDomain> DomainInstance;
	
	FHTNBuilder_Domain();
	FHTNBuilder_Domain(const TSharedPtr<FHTNDomain>& InDomain);

	void SetRootName(const FName InRootName) { RootTaskName = InRootName; }
	FHTNBuilder_CompositeTask& AddCompositeTask(const FName& TaskName);
	FHTNBuilder_PrimitiveTask& AddPrimitiveTask(const FName& TaskName);

	FHTNBuilder_CompositeTask* FindCompositeTask(const FName& TaskName);
	FHTNBuilder_PrimitiveTask* FindPrimitiveTask(const FName& TaskName);

	FHTNBuilder_CompositeTask* GetRootAsCompositeTask() { return FindCompositeTask(RootTaskName); }
	FHTNBuilder_PrimitiveTask* GetRootAsPrimitiveTask() { return FindPrimitiveTask(RootTaskName); }
	
	/**	Optimizes stored information. After the Domain is compiled
	 *	it's impossible to extend it
	 *	@return true if compilation was successful. False otherwise, look for warnings and errors in the log */
	bool Compile();
	/** Using DomainInstance information populate this domain builder instance */
	void Decompile();

	FString GetDebugDescription() const;

	FName RootTaskName;
	TMap<FName, FHTNBuilder_PrimitiveTask> PrimitiveTasks;
	TMap<FName, FHTNBuilder_CompositeTask> CompositeTasks;
};

