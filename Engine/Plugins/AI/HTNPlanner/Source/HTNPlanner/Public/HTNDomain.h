// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "HTNDomain.generated.h"

#define WITH_HTN_DEBUG !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

HTNPLANNER_API DECLARE_LOG_CATEGORY_EXTERN(LogHTNPlanner, Warning, All);

struct FHTNDomain;
struct FHTNWorldState;

namespace FHTNPolicy
{
	typedef uint16 FWSKey;
	typedef int32 FWSValue;
	typedef uint16 FTaskID;
	typedef uint16 FMethodID;
	typedef uint16 FActionID;
	typedef int32 FActionParameter;
	typedef uint8 FWSOperationID;
	const FWSValue DefaultValue = FWSValue(0);
	const FWSKey InvalidWSKey = MAX_uint16;
	const FTaskID InvalidTaskID = FTaskID(INDEX_NONE);
	const FMethodID InvalidMethodID = FMethodID(INDEX_NONE);
	const FActionID InvalidActionID = FActionID(MAX_uint16);
	const FWSOperationID InvalidWSOperation = FWSOperationID(MAX_uint8);
}

UENUM()
enum class EHTNWorldStateCheck : uint8
{
	Less,
	LessOrEqual,
	Equal,
	NotEqual,
	GreaterOrEqual,
	Greater,
	IsTrue,

	MAX UMETA(Hidden)
};

UENUM()
enum class EHTNWorldStateOperation : uint8
{
	Set,
	Increase,
	Decrease,

	MAX UMETA(Hidden)
};

template<typename TOperationEnum>
struct THTNWorldStateOperation
{
	typedef TOperationEnum FOperationEnum;

	const FHTNPolicy::FWSOperationID Operation;
	const FHTNPolicy::FWSKey KeyLeftHand;
	FHTNPolicy::FWSKey KeyRightHand;
	FHTNPolicy::FWSValue Value;
	
	THTNWorldStateOperation(const FHTNPolicy::FWSKey InKeyLeftHand = FHTNPolicy::InvalidWSKey, const FHTNPolicy::FWSOperationID InOperation = FHTNPolicy::InvalidWSOperation)
		: Operation(FHTNPolicy::FWSOperationID(InOperation)), KeyLeftHand(InKeyLeftHand), KeyRightHand(FHTNPolicy::InvalidWSKey), Value(FHTNPolicy::DefaultValue)
	{
	}

	THTNWorldStateOperation(const FHTNPolicy::FWSKey InKeyLeftHand, const FOperationEnum InOperation)
		: THTNWorldStateOperation(InKeyLeftHand, FHTNPolicy::FWSOperationID(InOperation))
	{
	}

	template<typename TAlternativeKey>
	THTNWorldStateOperation(const TAlternativeKey InKeyLeftHand, const FOperationEnum InOperation = FOperationEnum::MAX)
		: THTNWorldStateOperation(FHTNPolicy::FWSKey(InKeyLeftHand), InOperation)
	{
	}

	THTNWorldStateOperation<TOperationEnum>& SetRHSAsWSKey(const FHTNPolicy::FWSKey InKeyRightHand)
	{
		KeyRightHand = InKeyRightHand;
		return *this;
	}

	template<typename TAlternativeKey>
	THTNWorldStateOperation<TOperationEnum>& SetRHSAsWSKey(const TAlternativeKey InKeyRightHand)
	{
		return SetRHSAsWSKey(FHTNPolicy::FWSKey(InKeyRightHand));
	}

	THTNWorldStateOperation<TOperationEnum>& SetRHSAsValue(const FHTNPolicy::FWSValue InValue)
	{
		Value = InValue;
		return *this;
	}

	inline bool IsRHSAbsolute() const { return KeyRightHand == FHTNPolicy::InvalidWSKey; }

	bool IsValid() const { return (Operation != FHTNPolicy::InvalidWSOperation); }
};

typedef THTNWorldStateOperation<EHTNWorldStateCheck> FHTNCondition;
typedef THTNWorldStateOperation<EHTNWorldStateOperation> FHTNEffect;

namespace FHTNWorldStateOperations
{
	typedef bool(*FConditionFunctionPtr)(const FHTNPolicy::FWSValue*, const FHTNCondition& Condition);
	typedef void(*FOperationFunctionPtr)(FHTNPolicy::FWSValue*, const FHTNEffect& Effect);

	/** @return check ID, use it while building your conditions with the HTN builders */
	FHTNPolicy::FWSOperationID HTNPLANNER_API RegisterCustomCheckType(const FConditionFunctionPtr CustomConditionFunctionPtr, const FName& DebugName);
	/** @return check ID, use it while building your world state effects with the HTN builders */
	FHTNPolicy::FWSOperationID HTNPLANNER_API RegisterCustomOperationType(const FOperationFunctionPtr CustomOperationFunctionPtr, const FName& DebugName);
}

struct HTNPLANNER_API FHTNExecutableAction
{
	// operation ID (Move, Attack, Use, ...)
	FHTNPolicy::FActionID ActionID;
	// operand ID (Enemy, Table, CoverLocation, ...)
	FHTNPolicy::FActionParameter Parameter;

	FHTNExecutableAction()
		: ActionID(FHTNPolicy::InvalidActionID)
		, Parameter(0)
	{}

	bool operator==(const FHTNExecutableAction& Other) const
	{
		return ActionID == Other.ActionID && Parameter == Other.Parameter;
	}
};

struct HTNPLANNER_API FHTNPrimitiveTask : public FHTNExecutableAction
{
	// effect (applied to world state when this task is used)
	FHTNEffect* Effects;
	uint8 EffectsCount;

	FHTNPrimitiveTask()
		: Effects(nullptr)
		, EffectsCount(0)
	{}
};

struct HTNPLANNER_API FHTNMethod
{
	FHTNPolicy::FTaskID* Tasks;
	FHTNCondition* Conditions;
	uint8 TasksCount;
	uint8 ConditionsCount;

	FHTNMethod()
		: Tasks(nullptr)
		, Conditions(nullptr)
		, TasksCount(0)
		, ConditionsCount(0)
	{}
};

struct HTNPLANNER_API FHTNCompositeTask
{
	FHTNMethod* Methods;
	uint8 MethodsCount;

	FHTNCompositeTask()
		: Methods(nullptr)
		, MethodsCount(0)
	{}

	int32 FindSatisfiedMethod(const FHTNWorldState& WorldState, const int32 StartIndex = 0) const;
};

struct HTNPLANNER_API FHTNDomain : public TSharedFromThis<FHTNDomain>
{
	FHTNDomain(); 
	~FHTNDomain();

	UE_NONCOPYABLE(FHTNDomain);

	FHTNPolicy::FTaskID FindTaskID(const FName& TaskName) const;
	inline bool IsPrimitiveTask(const FHTNPolicy::FTaskID TaskID) const
	{
		return TaskID < FirstCompositeTaskID;
	}
	inline bool IsCompositeTask(const FHTNPolicy::FTaskID TaskID) const
	{
		return !IsPrimitiveTask(TaskID);
	}

	const FHTNPrimitiveTask& GetPrimitiveTask(const FHTNPolicy::FTaskID TaskID) const
	{
		return *((const FHTNPrimitiveTask*)(&RawData[TaskID]));
	}

	const FHTNCompositeTask& GetCompositeTask(const FHTNPolicy::FTaskID TaskID) const
	{
		return *((const FHTNCompositeTask*)(&RawData[TaskID]));
	}

	/** @note this information is available only if WITH_HTN_DEBUG */
	FName GetTaskName(const FHTNPolicy::FTaskID TaskID) const;

	FHTNPolicy::FTaskID GetRootTaskID() const { return RootTaskID; }
	bool IsEmpty() const { return (RawData == nullptr); }

protected:
	friend struct FHTNBuilder_Domain;
	void ReleaseRawData();

	uint8* RawData = nullptr;
	TMap<FName, FHTNPolicy::FTaskID> TaskNameMap;
	FHTNPolicy::FTaskID FirstCompositeTaskID;
	/**
	 * Root task is the default task we start planning from
	 */
	FHTNPolicy::FTaskID RootTaskID;
#if WITH_HTN_DEBUG
	TMap<FHTNPolicy::FTaskID, FName> TaskIDToName;
#endif // WITH_HTN_DEBUG

	void Reset();

};

struct HTNPLANNER_API FHTNWorldState
{	
	FHTNWorldState(const uint32 WorldStateSize = 128);
	~FHTNWorldState();

	void Reinit(const uint32 NewWorldStateSize = 128);

	bool CheckCondition(const FHTNCondition& Condition) const;
	inline bool CheckConditions(const FHTNCondition* Conditions, const int32 ConditionsCount) const
	{
		for (int32 ConditionIndex = 0; ConditionIndex < ConditionsCount; ++ConditionIndex)
		{
			if (CheckCondition(Conditions[ConditionIndex]) == false)
			{
				return false;
			}
		}
		return true;
	}

	void ApplyEffect(const FHTNEffect& Effect);
	inline void ApplyEffects(const FHTNEffect* Effects, const int32 EffectsCount)
	{
		for (int32 EffectIndex = 0; EffectIndex < EffectsCount; ++EffectIndex)
		{
			ApplyEffect(Effects[EffectIndex]);
		}
	}
	
	bool GetValue(const FHTNPolicy::FWSKey Key, FHTNPolicy::FWSValue& OutValue) const;
	FHTNPolicy::FWSValue GetValueUnsafe(const FHTNPolicy::FWSKey Key) const
	{
		return Values[Key];
	}

	bool SetValue(const FHTNPolicy::FWSKey Key, const FHTNPolicy::FWSValue InValue);
	void SetValueUnsafe(const FHTNPolicy::FWSKey Key, const FHTNPolicy::FWSValue InValue)
	{
		Values[Key] = InValue;
	}

	void Shrink() { Values.Shrink(); }

protected:
	TArray<FHTNPolicy::FWSValue> Values;
};
