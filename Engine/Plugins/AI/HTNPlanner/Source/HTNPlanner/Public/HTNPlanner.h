// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HTNDomain.h"

struct HTNPLANNER_API FHTNResult
{
	TArray<FHTNPolicy::FTaskID> TaskIDs;
	TArray<FHTNExecutableAction> ActionsSequence;

	void Reset()
	{
		TaskIDs.Reset();
		ActionsSequence.Reset();
	}
	void Set(const FHTNDomain& Domain, const TArray<FHTNPolicy::FTaskID>& TasksSequence);
};

struct HTNPLANNER_API FHTNRestorePoint
{
	enum 
	{
		InitialPlanSize = 5	// used to preallocate Plan array
	};

	FHTNWorldState WorldState;
	TArray<FHTNPolicy::FTaskID> Plan;
	int32 NextMethod;
	FHTNPolicy::FTaskID ActiveTask;

	explicit FHTNRestorePoint(const FHTNWorldState& InWorldState = FHTNWorldState())
		: WorldState(InWorldState), NextMethod(0), ActiveTask(FHTNPolicy::InvalidTaskID)
	{
		Plan.Reserve(InitialPlanSize);
	}

	FHTNRestorePoint(const FHTNRestorePoint& PreviousPoint, const FHTNPolicy::FTaskID CurrentTask, const int32 InNextMethod)
		: WorldState(PreviousPoint.WorldState), Plan(PreviousPoint.Plan), NextMethod(InNextMethod), ActiveTask(CurrentTask)
	{}
};

struct HTNPLANNER_API FHTNPlanner
{
	enum
	{
		InitialResporePoints = 5	// used to preallocate RestorePoints array
	};

	FHTNPlanner();
	bool GeneratePlan(const FHTNDomain& Domain, const FHTNWorldState& InitialWorldState, FHTNResult& Result, const FName& StartTaskName = NAME_None);
	const FHTNWorldState& GetWorldState() const { return CurrentState.WorldState; }

protected:
	FHTNWorldState& CurrentWorldState()	{ return CurrentState.WorldState; }
	void RecordDecomposition(const FHTNPolicy::FTaskID CurrentTask, const int32 MethodIndex) { RestorePoints.Push(FHTNRestorePoint(CurrentState, CurrentTask, MethodIndex)); }
	void RestoreDecomposition() { CurrentState = RestorePoints.Pop(EAllowShrinking::No); }
	bool CanRollBack() const { return RestorePoints.Num() > 0; }
	int32 GetMethodIndex() const { return CurrentState.NextMethod; }
	FHTNPolicy::FTaskID GetActiveTask() const { return CurrentState.ActiveTask; }
	const TArray<FHTNPolicy::FTaskID> GetPlan() const { return CurrentState.Plan; }
	void AddToPlan(const FHTNPolicy::FTaskID CurrentTask) { CurrentState.Plan.Push(CurrentTask); }
	
	void Reset()
	{
		//RestorePointsStored = 0;
		RestorePoints.Reset();
		CurrentState.Plan.Reset();
	}

	FHTNRestorePoint CurrentState;
	/** The array where consecutive restore points are stored. Note that 
	 *	the elements of this array are never getting destroyed (no destructor call) 
	 *	as long as the FHTNPlanner instance is alive. 
	 */
	TArray<FHTNRestorePoint> RestorePoints;
	/** Used for bookkeeping RestorePoints contents */
	int32 RestorePointsStored;
	TArray<FHTNPolicy::FTaskID> TasksToProcess;
};
