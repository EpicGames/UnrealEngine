// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
    public class HTNTestSuite : ModuleRules
    {
        public HTNTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicIncludePaths.AddRange(
                    new string[] {
                    }
                    );

            PublicDependencyModuleNames.AddRange(
                new string[] {
                        "Core",
                        "CoreUObject",
                        "Engine",
                        "AIModule",
                        "HTNPlanner",
                        "AITestSuite",
                }
                );

            DynamicallyLoadedModuleNames.AddRange(
                new string[] {
                    // ... add any modules that your module loads dynamically here ...
                }
                );

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.Add("EditorFramework");
                PrivateDependencyModuleNames.Add("UnrealEd");
            }
        }
    }
}