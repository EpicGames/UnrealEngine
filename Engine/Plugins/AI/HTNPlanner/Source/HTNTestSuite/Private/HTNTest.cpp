// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTNBuilder.h"
#include "MockHTN.h"
#include "AITestsCommon.h"
#include "Debug/HTNDebug.h"

#define LOCTEXT_NAMESPACE "AITestSuite_HTNTest"

// missing tests:
//  testing invalid conditions
//	testing specific scenarios with known plans 
//	test step by step generating the same plan as GeneratePlan
//	using memory counters see if anything's leaking

struct FHTNTestBase : public FAITestBase
{
	FHTNBuilder_Domain DomainBuilder;
	FHTNWorldState WorldState;
	FHTNPlanner Planner;

	void PopulateWorldState()
	{
		for (int32 WSIndex = 0; WSIndex < int32(EMockHTNWorldState::MAX); ++WSIndex)
		{
			// setting every key to it's numerical index value
			WorldState.SetValueUnsafe(static_cast<FHTNPolicy::FWSKey>(WSIndex), WSIndex);
		}
	}

	void PopulateDomain(const bool bCompile = true)
	{
		DomainBuilder.SetRootName(TEXT("Root"));
		{
			FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(TEXT("Root"));
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod(
					TArray<FHTNCondition>({
					FHTNCondition(EMockHTNWorldState::EnemyHealth, EHTNWorldStateCheck::Greater).SetRHSAsValue(0)
					, FHTNCondition(EMockHTNWorldState::EnemyActor, EHTNWorldStateCheck::IsTrue)
				}));
				MethodsBuilder.AddTask(TEXT("AttackEnemy"));
			}
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
				MethodsBuilder.AddTask(TEXT("FindPatrolPoint"));
				MethodsBuilder.AddTask(TEXT("NavigateToMoveDestination"));
			}
		}
		{
			FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(TEXT("AttackEnemy"));
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod(FHTNCondition(EMockHTNWorldState::HasWeapon, EHTNWorldStateCheck::IsTrue));
				MethodsBuilder.AddTask(TEXT("NavigateToEnemy"));
				MethodsBuilder.AddTask(TEXT("UseWeapon"));
				MethodsBuilder.AddTask(TEXT("Root"));
			}
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
				MethodsBuilder.AddTask(TEXT("FindWeapon"));
				MethodsBuilder.AddTask(TEXT("NavigateToWeapon"));
				MethodsBuilder.AddTask(TEXT("PickUp"));
				MethodsBuilder.AddTask(TEXT("AttackEnemy"));
			}
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("FindPatrolPoint"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::FindPatrolPoint, EMockHTNWorldState::MoveDestination);
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("FindWeapon"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::FindWeapon, EMockHTNWorldState::PickupLocation);
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("NavigateToMoveDestination"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::NavigateTo, EMockHTNWorldState::MoveDestination);	// Local Variables?
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::CurrentLocation, EHTNWorldStateOperation::Set).SetRHSAsWSKey(EMockHTNWorldState::MoveDestination));
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("NavigateToEnemy"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::NavigateTo, EMockHTNWorldState::EnemyActor);
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::CurrentLocation, EHTNWorldStateOperation::Set).SetRHSAsWSKey(EMockHTNWorldState::EnemyActor));
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::CanSeeEnemy, EHTNWorldStateOperation::Set).SetRHSAsValue(1));
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("NavigateToWeapon"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::NavigateTo, EMockHTNWorldState::PickupLocation);
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::CurrentLocation, EHTNWorldStateOperation::Set).SetRHSAsWSKey(EMockHTNWorldState::PickupLocation));
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("PickUp"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::PickUp, EMockHTNWorldState::PickupLocation);
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::HasWeapon, EHTNWorldStateOperation::Set).SetRHSAsValue(1));
		}
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(TEXT("UseWeapon"));
			PrimitiveTaskBuilder.SetOperator(EMockHTNTaskOperator::UseWeapon, EMockHTNWorldState::EnemyActor);
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::Ammo, EHTNWorldStateOperation::Decrease).SetRHSAsValue(1));
			PrimitiveTaskBuilder.AddEffect(FHTNEffect(EMockHTNWorldState::EnemyHealth, EHTNWorldStateOperation::Decrease).SetRHSAsValue(1));
		}

		if (bCompile)
		{
			DomainBuilder.Compile();
		}
	}
};

//////////////////////////////////////////////////////////////////////////

struct FAITest_HTNDomainBuilderBasics : public FAITestBase
{
	virtual bool InstantTest() override
	{
		FHTNBuilder_Domain DomainBuilder;

		AITEST_TRUE("Initially DomainBuilder instance should be empty", DomainBuilder.CompositeTasks.Num() == 0 && DomainBuilder.PrimitiveTasks.Num() == 0);

		const uint32 CompointTasksCount = 5;
		for (int32 CompositeTaskIndex = 0; CompositeTaskIndex < CompointTasksCount; ++CompositeTaskIndex)
		{
			FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(*FString::Printf(TEXT("c_%d"), CompositeTaskIndex));
			for (int32 MethodIndex = 0; MethodIndex < (CompositeTaskIndex % 3); ++MethodIndex)
			{
				FHTNBuilder_Method* MethodsBuilder = nullptr;

				if ((CompositeTaskIndex + MethodIndex) != 0)
				{
					TArray<FHTNCondition> Conditions;
					for (int32 ConditionIndex = 0; ConditionIndex < (CompositeTaskIndex % 3); ++ConditionIndex)
					{
						Conditions.Add(FHTNCondition(ConditionIndex, EHTNWorldStateCheck(ConditionIndex)));
					}

					MethodsBuilder = &CompositeTaskBuilder.AddMethod(Conditions);
				}
				else
				{
					// this is testing the non-array method
					MethodsBuilder = &CompositeTaskBuilder.AddMethod(FHTNCondition(0, EHTNWorldStateCheck(0)));
				}

				for (int32 TaskIndex = 0; TaskIndex < (CompositeTaskIndex % 4); ++TaskIndex)
				{
					MethodsBuilder->AddTask(*FString::Printf(TEXT("t_%d"), CompositeTaskIndex));
				}
			}
		}

		for (int32 CompositeTaskIndex = 0; CompositeTaskIndex < DomainBuilder.CompositeTasks.Num(); ++CompositeTaskIndex)
		{
			FHTNBuilder_CompositeTask* CompositeTaskBuilder = DomainBuilder.CompositeTasks.Find(*FString::Printf(TEXT("c_%d"), CompositeTaskIndex));
			AITEST_TRUE(FString::Printf(TEXT("Failed to find Composite task c_%d that has just been added"), CompositeTaskIndex), CompositeTaskBuilder != nullptr);

			if (CompositeTaskBuilder != nullptr)
			{
				AITEST_TRUE(FString::Printf(TEXT("Method count mismatch for c_%d"), CompositeTaskIndex), CompositeTaskBuilder->Methods.Num() == (CompositeTaskIndex % 3));
				for (int32 MethodIndex = 0; MethodIndex < CompositeTaskBuilder->Methods.Num(); ++MethodIndex)
				{
					if ((CompositeTaskIndex + MethodIndex) != 0)
					{
						AITEST_TRUE(FString::Printf(TEXT("Condition count mismatch for c_%d[%d] method (array path)"), CompositeTaskIndex, MethodIndex)
							, CompositeTaskBuilder->Methods[MethodIndex].Conditions.Num() == (CompositeTaskIndex % 3));
					}
					else
					{
						AITEST_TRUE(FString::Printf(TEXT("Condition count mismatch for c_%d[%d] method (single instance)"), CompositeTaskIndex, MethodIndex)
							, CompositeTaskBuilder->Methods[MethodIndex].Conditions.Num() == 1);
					}

					AITEST_TRUE(FString::Printf(TEXT("Task count mismatch for c_%d[%d] method"), CompositeTaskIndex, MethodIndex)
						, CompositeTaskBuilder->Methods[MethodIndex].Tasks.Num() == (CompositeTaskIndex % 4));
				}
			}			
		}

		const uint32 PrimitiveTasksCount = 4;
		for (int32 PrimitiveTaskIndex = 0; PrimitiveTaskIndex < PrimitiveTasksCount; ++PrimitiveTaskIndex)
		{
			FHTNBuilder_PrimitiveTask& PrimitiveTaskBuilder = DomainBuilder.AddPrimitiveTask(*FString::Printf(TEXT("p_%d"), PrimitiveTaskIndex));
			PrimitiveTaskBuilder.SetOperator(PrimitiveTaskIndex, PrimitiveTaskIndex * 2);
			for (int32 EffectIndex = 0; EffectIndex < (PrimitiveTaskIndex % 3); ++EffectIndex)
			{
				PrimitiveTaskBuilder.AddEffect(FHTNEffect(EffectIndex, EHTNWorldStateOperation::Set));
			}
		}

		AITEST_TRUE("Wrong number of primitive tasks added", DomainBuilder.PrimitiveTasks.Num() == PrimitiveTasksCount);
		for (int32 PrimitiveTaskIndex = 0; PrimitiveTaskIndex < PrimitiveTasksCount; ++PrimitiveTaskIndex)
		{
			FHTNBuilder_PrimitiveTask* PrimitiveTaskBuilder = DomainBuilder.PrimitiveTasks.Find(*FString::Printf(TEXT("p_%d"), PrimitiveTaskIndex));
			AITEST_TRUE(FString::Printf(TEXT("Failed to find primitive task p_%d that has just been added"), PrimitiveTaskIndex), PrimitiveTaskBuilder != nullptr);

			if (PrimitiveTaskBuilder)
			{
				AITEST_TRUE(FString::Printf(TEXT("Primitive task p_%d operator is wrong"), PrimitiveTasksCount), PrimitiveTaskBuilder->ActionID == PrimitiveTaskIndex);
				AITEST_TRUE(FString::Printf(TEXT("Primitive task p_%d effects count is wrong"), PrimitiveTasksCount), PrimitiveTaskBuilder->Effects.Num() == (PrimitiveTaskIndex % 3));
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNDomainBuilderBasics, "System.AI.HTN.DomainBuilderBasics")

struct FAITest_HTNBuildDomain : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		PopulateDomain();

		AITEST_TRUE("DomainBuilder stores wrong number of primitive tasks", DomainBuilder.PrimitiveTasks.Num() == 7);
		AITEST_TRUE("DomainBuilder stores wrong number of Composite tasks", DomainBuilder.CompositeTasks.Num() == 2);
		
		FHTNBuilder_CompositeTask* RootBuilder = DomainBuilder.GetRootAsCompositeTask();
		AITEST_TRUE("Root task should be set", RootBuilder != nullptr);
		if (RootBuilder && RootBuilder->Methods.Num() > 0)
		{
			AITEST_TRUE("Root task\'s method [0] should be configured to have two conditions", RootBuilder->Methods[0].Conditions.Num() == 2);
		}
		
		TArray<FHTNBuilder_CompositeTask> CompositeTasks;
		DomainBuilder.CompositeTasks.GenerateValueArray(CompositeTasks);
		AITEST_TRUE("DomainBuilder stores wrong number of methods for the first Composite task", CompositeTasks[0].Methods.Num() == 2);
		AITEST_TRUE("DomainBuilder stores wrong number of methods for the second Composite task", CompositeTasks[1].Methods.Num() == 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNBuildDomain, "System.AI.HTN.BuildDomain")

struct FAITest_HTNPlanning : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		FHTNResult Result;
		
		Planner.GeneratePlan(*(DomainBuilder.DomainInstance), WorldState, Result);
		AITEST_TRUE("Planning with an empty domain should result in an empty plan", Result.TaskIDs.Num() == 0);

		PopulateDomain();

		Planner.GeneratePlan(*(DomainBuilder.DomainInstance), WorldState, Result);

		TArray<FHTNBuilder_CompositeTask> CompositeTasks;
		DomainBuilder.CompositeTasks.GenerateValueArray(CompositeTasks);
		// Patrol plan
		AITEST_TRUE("Patrol plan should be same length as the last methods of the root Composite task", CompositeTasks[0].Methods.Last().Tasks.Num() == Result.TaskIDs.Num());
		for (int32 TaskIndex = 0; TaskIndex < Result.TaskIDs.Num(); ++TaskIndex)
		{
			const FHTNPolicy::FTaskID TaskID = Result.TaskIDs[TaskIndex];
			AITEST_TRUE("Patrol plan element mismatch", TaskID == DomainBuilder.DomainInstance->FindTaskID(CompositeTasks[0].Methods.Last().Tasks[TaskIndex]));
		}

		const FHTNDomain EmptyDomain;
		Planner.GeneratePlan(EmptyDomain, WorldState, Result);
		AITEST_TRUE("Reusing previous planning result with an empty domain should result in an empty plan", Result.TaskIDs.Num() == 0);

		WorldState.ApplyEffect(FHTNEffect(EMockHTNWorldState::EnemyHealth, EHTNWorldStateOperation::Set).SetRHSAsValue(1));
		WorldState.ApplyEffect(FHTNEffect(EMockHTNWorldState::EnemyActor, EHTNWorldStateOperation::Set).SetRHSAsValue(1));
		Planner.GeneratePlan(*(DomainBuilder.DomainInstance), WorldState, Result);
		AITEST_TRUE("", Result.TaskIDs.Num() > 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNPlanning, "System.AI.HTN.Planning")

struct FAITest_HTNPlanningRollback : public FHTNTestBase
{
	virtual bool InstantTest() override
	{		
		FHTNResult Result;

		// build a domain that will force rolling back
		// first method should get accepted in the first planner step
		//		then one of the tasks it consists of should be a Composite task, that fails its condition
		// Note: WorldState is populated with 0 values and has 128 keys (by default)
		{
			FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(NAME_None);	// root
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
				MethodsBuilder.AddTask(TEXT("FailedComposite"));
			}
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
				MethodsBuilder.AddTask(TEXT("SuccessfulComposite"));
			}
		}
		{
			FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(TEXT("FailedComposite"));
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod(FHTNCondition(0, EHTNWorldStateCheck::Greater).SetRHSAsValue(0));
				MethodsBuilder.AddTask(TEXT("DummyPrimitive1"));
			}
		}
		{
			FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(TEXT("SuccessfulComposite"));
			{
				FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod(FHTNCondition(0, EHTNWorldStateCheck::Equal).SetRHSAsValue(0));
				MethodsBuilder.AddTask(TEXT("DummyPrimitive2"));
			}
		}
		DomainBuilder.AddPrimitiveTask(TEXT("DummyPrimitive1"));
		const FName DummyPrimitiveName2 = TEXT("DummyPrimitive2");
		DomainBuilder.AddPrimitiveTask(DummyPrimitiveName2);
		
		DomainBuilder.Compile();

		Planner.GeneratePlan(*(DomainBuilder.DomainInstance), WorldState, Result);
		AITEST_TRUE("First Rollback plan should consist of one task, DummyPrimitive2", Result.TaskIDs.Num() == 1
			&& FHTNDebug::GetTaskName(DomainBuilder, Result.TaskIDs[0]) == DummyPrimitiveName2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNPlanningRollback, "System.AI.HTN.PlanningRollback")

struct FAITest_HTNDecompileDomain : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		AITEST_TRUE("Compiling an empty domain is allowed", DomainBuilder.Compile());

		PopulateDomain();

		FHTNBuilder_Domain DomainBuilder2(DomainBuilder.DomainInstance);
		DomainBuilder2.Decompile();
		const FString OriginalDescription = DomainBuilder.GetDebugDescription();
		const FString DecompiledDescription = DomainBuilder2.GetDebugDescription();

		AITEST_TRUE("Decompilation should result in identical DomainBuilder", OriginalDescription.Equals(DecompiledDescription));
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNDecompileDomain, "System.AI.HTN.DomainDecompilation")

struct FAITest_HTNDomainCompilationIssues : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		const FName MissingTaskName = TEXT("MissingTask");

		FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(TEXT("Root"));	// root
		{
			FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
			MethodsBuilder.AddTask(MissingTaskName);
		}

		AITEST_FALSE("Domain with missing tasks should not compile", DomainBuilder.Compile());
		AITEST_TRUE("Domain should be empty after a failed compilation", DomainBuilder.DomainInstance->IsEmpty());

		DomainBuilder.AddPrimitiveTask(MissingTaskName);
		AITEST_TRUE("After adding missing task domain should compile just fine", DomainBuilder.Compile());
		AITEST_FALSE("Domain should not be empty after a successful compilation", DomainBuilder.DomainInstance->IsEmpty());
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNDomainCompilationIssues, "System.AI.HTN.DomainDecompilationIssues")

struct FAITest_HTNWorldRepresentation : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		for (int32 WSIndex = 0; WSIndex < int32(EMockHTNWorldState::MAX); ++WSIndex)
		{
			FHTNPolicy::FWSValue Value;
			AITEST_TRUE("Retrieving known values from the WorldState instance", WorldState.GetValue(static_cast<FHTNPolicy::FWSKey>(WSIndex), Value) && Value == FHTNPolicy::DefaultValue);
		}

		PopulateWorldState();
		
		const uint32 ReferenceValue = 3;
		for (int32 WSIndex = 0; WSIndex < int32(EMockHTNWorldState::MAX); ++WSIndex)
		{
			FHTNPolicy::FWSValue Value;
			AITEST_TRUE("Retrieving known values from the WorldState instance", WorldState.GetValue(static_cast<FHTNPolicy::FWSKey>(WSIndex), Value) && Value == WSIndex);

			for (int32 OpIndex = 0; OpIndex < int32(EHTNWorldStateCheck::MAX); ++OpIndex)
			{
				const EHTNWorldStateCheck OpCode = EHTNWorldStateCheck(OpIndex);
				bool bExpectedResult = false;

				switch (OpCode)
				{
				case EHTNWorldStateCheck::Less:
					bExpectedResult = Value < ReferenceValue;
					break;
				case EHTNWorldStateCheck::LessOrEqual:
					bExpectedResult = Value <= ReferenceValue;
					break;
				case EHTNWorldStateCheck::Equal:
					bExpectedResult = Value == ReferenceValue;
					break;
				case EHTNWorldStateCheck::NotEqual:
					bExpectedResult = Value != ReferenceValue;
					break;
				case EHTNWorldStateCheck::GreaterOrEqual:
					bExpectedResult = Value >= ReferenceValue;
					break;
				case EHTNWorldStateCheck::Greater:
					bExpectedResult = Value > ReferenceValue;
					break;
				case EHTNWorldStateCheck::IsTrue:
					bExpectedResult = (Value != 0);
					break;
				default:
					AITEST_TRUE("Unhanled operation ID!", true);
				}

				FString Message = FString::Printf(TEXT("Testing %s on %d"), *FHTNDebug::HTNWorldStateCheckToString(OpCode), Value);
				AITEST_TRUE(Message, WorldState.CheckCondition(FHTNCondition(WSIndex, OpCode).SetRHSAsValue(ReferenceValue)) == bExpectedResult);
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNWorldRepresentation, "System.AI.HTN.WorldRepresentation")

struct FAITest_HTNCondition : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		PopulateWorldState();

		const uint32 ReferenceValue = 3;

		for (int32 WSIndex = 0; WSIndex < int32(EMockHTNWorldState::MAX); ++WSIndex)
		{
			for (int32 Value = 0; Value < int32(EMockHTNWorldState::MAX); ++Value)
			{
				const FHTNPolicy::FWSValue AsValue = FHTNPolicy::FWSValue(Value);
				const FHTNPolicy::FWSKey AsKey = FHTNPolicy::FWSKey(Value);

				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] < %d"), WSIndex, AsValue)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::Less).SetRHSAsValue(AsValue)) == (WSIndex < AsValue));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] <= %d"), WSIndex, AsValue)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::LessOrEqual).SetRHSAsValue(AsValue)) == (WSIndex <= AsValue));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] == %d"), WSIndex, AsValue)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::Equal).SetRHSAsValue(AsValue)) == (WSIndex == AsValue));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] != %d"), WSIndex, AsValue)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::NotEqual).SetRHSAsValue(AsValue)) == (WSIndex != AsValue));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] >= %d"), WSIndex, AsValue)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::GreaterOrEqual).SetRHSAsValue(AsValue)) == (WSIndex >= AsValue));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] > %d"), WSIndex, AsValue)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::Greater).SetRHSAsValue(AsValue)) == (WSIndex > AsValue));

				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] < WS[%d]"), WSIndex, AsKey)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::Less).SetRHSAsWSKey(AsKey)) == (WSIndex < AsKey));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] <= WS[%d]"), WSIndex, AsKey)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::LessOrEqual).SetRHSAsWSKey(AsKey)) == (WSIndex <= AsKey));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] == WS[%d]"), WSIndex, AsKey)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::Equal).SetRHSAsWSKey(AsKey)) == (WSIndex == AsKey));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] != WS[%d]"), WSIndex, AsKey)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::NotEqual).SetRHSAsWSKey(AsKey)) == (WSIndex != AsKey));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] >= WS[%d]"), WSIndex, AsKey)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::GreaterOrEqual).SetRHSAsWSKey(AsKey)) == (WSIndex >= AsKey));
				AITEST_TRUE(FString::Printf(TEXT("Condition WS[%d] > WS[%d]"), WSIndex, AsKey)
					, WorldState.CheckCondition(FHTNCondition(WSIndex, EHTNWorldStateCheck::Greater).SetRHSAsWSKey(AsKey)) == (WSIndex > AsKey));
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNCondition, "System.AI.HTN.Condition")

struct FAITest_HTNMethodSelection : public FAITestBase 
{
	virtual bool InstantTest() override
	{
		FHTNDomain Domain;

		//FHTNCompositeTask& RootCompositeTask = Domain.AddCompositeTask(TEXT("Root"));
		// do planning and come up with empty plan

		// INJECTION
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNMethodSelection, "System.AI.HTN.MethodSelection")

struct FAITest_HTNTrivialPlanning : public FHTNTestBase 
{
	virtual bool InstantTest() override
	{
		//FHTNDomain Domain;

		TArray<FHTNCondition> Conditions;
		//FHTNMethodSelection MethodSelection(0, Conditions);


		//FHTNCompositeTask& RootCompositeTask = Domain.AddCompositeTask(TEXT("Root"));
		// do planning and come up with empty plan
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNTrivialPlanning, "System.AI.HTN.TrivialPlanning")

struct FAITest_HTNExtendingDomain : public FAITestBase
{
	virtual bool InstantTest() override
	{
		//FHTNDomain Domain;

		//FHTNCompositeTask& RootCompositeTask = Domain.AddCompositeTask(TEXT("Root"));
		// do planning and come up with empty plan

		// INJECTION
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNExtendingDomain, "System.AI.HTN.ExtendingDomain")

struct FAITest_HTNCustomWSCheck : public FHTNTestBase
{
	static bool bCheckFunctionRun;
	static bool CustomCheck(const FHTNPolicy::FWSValue* Values, const FHTNCondition& Condition)
	{
		bCheckFunctionRun = true;
		return true;
	}

	virtual bool InstantTest() override
	{
		const uint32 CustomCheckID = FHTNWorldStateOperations::RegisterCustomCheckType(&FAITest_HTNCustomWSCheck::CustomCheck, TEXT("CustomCheck")); 

		FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(NAME_None);	// root
		{
			FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod(FHTNCondition(0, IntCastChecked<FHTNPolicy::FWSOperationID>(CustomCheckID)));
			MethodsBuilder.AddTask(TEXT("Task1"));
		}
		FHTNBuilder_PrimitiveTask& PrimitiveTask = DomainBuilder.AddPrimitiveTask(TEXT("Task1"));
		
		DomainBuilder.Compile();
		
		FHTNResult Result;
		Planner.GeneratePlan(*DomainBuilder.DomainInstance, WorldState, Result);

		AITEST_TRUE("Custom check has been exeduted", bCheckFunctionRun);
		AITEST_TRUE("The custom check should allow for construction of the plan", Result.TaskIDs.Num() == 1);

		return true;
	}
};
bool FAITest_HTNCustomWSCheck::bCheckFunctionRun = false;
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNCustomWSCheck, "System.AI.HTN.CustomWSCheck")

struct FAITest_HTNCustomWSOperation : public FHTNTestBase
{
	static uint32 StaticValue;
	static void CustomOperation(FHTNPolicy::FWSValue* Values, const FHTNEffect& Effect)
	{
		++StaticValue;
		Values[Effect.KeyLeftHand] = StaticValue * 1024;
	}

	virtual bool InstantTest() override
	{
		const uint32 CustomOperationID = FHTNWorldStateOperations::RegisterCustomOperationType(&FAITest_HTNCustomWSOperation::CustomOperation, TEXT("CustomOperation"));

		FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(NAME_None);	// root
		{
			FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
			MethodsBuilder.AddTask(TEXT("Task1"));
		}
		FHTNBuilder_PrimitiveTask& PrimitiveTask = DomainBuilder.AddPrimitiveTask(TEXT("Task1"));
		PrimitiveTask.AddEffect(FHTNEffect(0, IntCastChecked<FHTNPolicy::FWSOperationID>(CustomOperationID)));
		PrimitiveTask.AddEffect(FHTNEffect(2, IntCastChecked<FHTNPolicy::FWSOperationID>(CustomOperationID)));

		DomainBuilder.Compile();

		FHTNResult Result;
		Planner.GeneratePlan(*DomainBuilder.DomainInstance, WorldState, Result);

		AITEST_TRUE("Custom Operation has been exeduted", StaticValue == 2);
		AITEST_TRUE("Checking custom effect on key 0", Planner.GetWorldState().GetValueUnsafe(0) == 1024);
		AITEST_TRUE("Checking custom effect on key 0", Planner.GetWorldState().GetValueUnsafe(2) == 1024 * 2);

		return true;
	}
};
uint32 FAITest_HTNCustomWSOperation::StaticValue = 0;
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNCustomWSOperation, "System.AI.HTN.CustomWSOperation")

//	compare contents of TaskIDs in HTNResult to the operators it contains
struct FAITest_HTNOperatorsOfGeneratedPlan : public FHTNTestBase
{
	virtual bool InstantTest() override
	{
		FHTNBuilder_CompositeTask& CompositeTaskBuilder = DomainBuilder.AddCompositeTask(NAME_None);
		{
			FHTNBuilder_Method& MethodsBuilder = CompositeTaskBuilder.AddMethod();
			MethodsBuilder.AddTask(TEXT("Task2"));
			MethodsBuilder.AddTask(TEXT("Task1"));
		}
		FHTNBuilder_PrimitiveTask& PrimitiveTask1 = DomainBuilder.AddPrimitiveTask(TEXT("Task1"));
		PrimitiveTask1.SetOperator(1, 2);
		FHTNBuilder_PrimitiveTask& PrimitiveTask2 = DomainBuilder.AddPrimitiveTask(TEXT("Task2"));
		PrimitiveTask2.SetOperator(3, 4);
		
		DomainBuilder.Compile();

		FHTNResult Result;
		Planner.GeneratePlan(*DomainBuilder.DomainInstance, WorldState, Result);
		
		AITEST_TRUE("Plan should contain two elements", Result.TaskIDs.Num() == 2);
		if (Result.TaskIDs.Num() >= 2)
		{
			const FHTNPolicy::FTaskID TaskID1 = DomainBuilder.DomainInstance->FindTaskID(TEXT("Task1"));
			const FHTNPolicy::FTaskID TaskID2 = DomainBuilder.DomainInstance->FindTaskID(TEXT("Task2"));
			AITEST_TRUE("Task2 should be the first one", Result.TaskIDs[0] == TaskID2);
			AITEST_TRUE("Task2 action should be the first one", Result.ActionsSequence[0].ActionID == 3 && Result.ActionsSequence[0].Parameter == 4);
			AITEST_TRUE("Task1 should be the second one", Result.TaskIDs[1] == TaskID1);
			AITEST_TRUE("Task1 action should be the first one", Result.ActionsSequence[1].ActionID == 1 && Result.ActionsSequence[1].Parameter == 2);


		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_HTNOperatorsOfGeneratedPlan, "System.AI.HTN.OperatorsOfGeneratedPlan")

//----------------------------------------------------------------------//
// Component tests
//----------------------------------------------------------------------//
typedef FAITest_SimpleComponentBasedTest<UMockHTNComponent> FAITest_HTNComponentTest;

#undef LOCTEXT_NAMESPACE
