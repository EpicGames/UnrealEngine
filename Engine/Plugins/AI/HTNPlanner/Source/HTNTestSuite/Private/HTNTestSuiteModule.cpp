// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTNTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "HTNTestSuite"

class FHTNTestSuiteModule : public IHTNTestSuiteModule
{
};

IMPLEMENT_MODULE(FHTNTestSuiteModule, HTNTestSuite)

#undef LOCTEXT_NAMESPACE
