// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "AI/HTNBrainComponent.h"
#include "MockHTN.generated.h"

enum class EMockHTNWorldState : uint8
{
	EnemyHealth,
	EnemyActor,
	Ammo,
	AbilityRange,
	HasWeapon,
	MoveDestination,
	PickupLocation,
	CurrentLocation,
	CanSeeEnemy,

	MAX
};

enum class EMockHTNTaskOperator : uint8
{
	DummyOperation,
	FindPatrolPoint,
	FindWeapon,
	NavigateTo,
	PickUp,
	UseWeapon,

	MAX
};

UCLASS()
class UMockHTNComponent : public UHTNBrainComponent
{
	GENERATED_BODY()

public:
	//int32 GetTaskPriorityQueueSize() const { return TaskPriorityQueue.Num(); }
};
